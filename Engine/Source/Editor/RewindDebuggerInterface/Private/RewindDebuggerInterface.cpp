// Copyright Epic Games, Inc. All Rights Reserved.

#include "IRewindDebugger.h"
#include "IRewindDebuggerDoubleClickHandler.h"
#include "IRewindDebuggerExtension.h"
#include "IRewindDebuggerTrackCreator.h"
#include "IRewindDebuggerViewCreator.h"
#include "UObject/NameTypes.h"

DEFINE_LOG_CATEGORY(LogRewindDebugger)

const FName IRewindDebuggerExtension::ModularFeatureName = "RewindDebuggerExtension";
const FName IRewindDebuggerViewCreator::ModularFeatureName = "RewindDebuggerViewCreator";
const FName IRewindDebuggerDoubleClickHandler::ModularFeatureName = "RewindDebuggerDoubleClickHandler";

namespace RewindDebugger
{
	const FName IRewindDebuggerTrackCreator::ModularFeatureName = "RewindDebuggerTrackCreator";
}

IRewindDebugger* IRewindDebugger::InternalInstance = nullptr;

IRewindDebugger::~IRewindDebugger()
{
}

IRewindDebugger::IRewindDebugger()
{
}

IRewindDebugger* IRewindDebugger::Instance()
{
	return InternalInstance;
}