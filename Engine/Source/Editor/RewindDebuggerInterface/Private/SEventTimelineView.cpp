// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEventTimelineView.h"
#include "SSimpleTimeSlider.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "SEventTimelineView"

void SEventTimelineView::Construct(const SEventTimelineView::FArguments& InArgs)
{
	ViewRange = InArgs._ViewRange;
	DesiredSize = InArgs._DesiredSize;
	EventData = InArgs._EventData;

	// set clipping on by default, since the OnPaint function is drawing outside the bounds
	// Clipping = EWidgetClipping::ClipToBounds;
	EventBrush = FAppStyle::GetBrush("Sequencer.KeyDiamond");
	EventBorderBrush = FAppStyle::GetBrush("Sequencer.KeyDiamondBorder");
}

int32 SEventTimelineView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, const int32 LayerId, const FWidgetStyle& InWidgetStyle, const bool bParentEnabled) const
{
	const int32 NewLayer = PaintEvents(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	return FMath::Max(NewLayer, SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, NewLayer, InWidgetStyle, ShouldBeEnabled(bParentEnabled)));
}

FVector2D SEventTimelineView::ComputeDesiredSize(float) const
{
	return DesiredSize.Get();
}

int32 SEventTimelineView::PaintEvents(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const TRange<double> DebugTimeRange = ViewRange.Get();

	const SSimpleTimeSlider::FScrubRangeToScreen RangeToScreen(DebugTimeRange, AllottedGeometry.GetLocalSize());

	if (const TSharedPtr<FTimelineEventData> Event = EventData.Get())
	{
		const TConstArrayView<FTimelineEventData::FEventPoint> EventPoints = Event->EventPoints;
		const int32 NumPoints = EventPoints.Num();

		double PrevPointTime = 0;
		int32 OverlappingPointCount = 0;

		if (NumPoints > 0)
		{
			for (int32 i = 0; i < NumPoints; i++)
			{
				const FTimelineEventData::FEventPoint& Point = EventPoints[i];

				FVector2D EventSize = EventBrush->GetImageSize();

				float X = RangeToScreen.InputToLocalX(Point.Time);
				X = X - EventSize.X / 2;
				float Y = (AllottedGeometry.Size.Y - EventSize.Y) / 2;
				if (Point.Time == PrevPointTime)
				{
					static constexpr int32 OverlapOffsetAmount = 2;
					static constexpr int32 MaxOverlappingEvents = 2;

					if (OverlappingPointCount > MaxOverlappingEvents)
					{
						continue;
					}

					OverlappingPointCount++;
					Y += OverlapOffsetAmount * OverlappingPointCount;
				}
				else
				{
					OverlappingPointCount = 0;
				}

				PrevPointTime = Point.Time;

				FSlateDrawElement::MakeBox(OutDrawElements, LayerId,
					AllottedGeometry.ToPaintGeometry(EventSize, FSlateLayoutTransform(FVector2f(X, Y))), EventBrush, ESlateDrawEffect::None, Point.Color);

				FSlateDrawElement::MakeBox(OutDrawElements, LayerId,
					AllottedGeometry.ToPaintGeometry(EventSize, FSlateLayoutTransform(FVector2f(X, Y))), EventBorderBrush, ESlateDrawEffect::None, FLinearColor::Black);
			}
		}

		const TConstArrayView<FTimelineEventData::FEventWindow> EventWindows = Event->EventWindows;
		const int32 NumWindows = EventWindows.Num();

		const FSlateBrush* Brush = FAppStyle::GetBrush("Sequencer.SectionArea.Background");

		if (NumWindows > 0)
		{
			for (int32 i = 0; i < NumWindows; i++)
			{
				const FTimelineEventData::FEventWindow& Window = EventWindows[i];

				FVector2D EventSize = EventBrush->GetImageSize();

				const float XStart = RangeToScreen.InputToLocalX(Window.TimeStart);
				const float XStartDiamond = XStart - EventSize.X / 2;
				const float XEnd = RangeToScreen.InputToLocalX(Window.TimeEnd);
				const float XEndDiamond = XEnd - EventSize.X / 2;
				const float Y = (AllottedGeometry.Size.Y - EventSize.Y) / 2;

				// window bar
				FSlateDrawElement::MakeBox(OutDrawElements, LayerId++,
					AllottedGeometry.ToPaintGeometry(FVector2D(XEnd - XStart, EventSize.Y - 2), FSlateLayoutTransform(FVector2D(XStart, Y + 1))), Brush, ESlateDrawEffect::None, Window.Color);

				// key diamond at start
				FSlateDrawElement::MakeBox(OutDrawElements, LayerId,
					AllottedGeometry.ToPaintGeometry(EventSize, FSlateLayoutTransform(FVector2D(XStartDiamond, Y))), EventBrush, ESlateDrawEffect::None, Window.Color);

				FSlateDrawElement::MakeBox(OutDrawElements, LayerId,
					AllottedGeometry.ToPaintGeometry(EventSize, FSlateLayoutTransform(FVector2D(XStartDiamond, Y))), EventBorderBrush, ESlateDrawEffect::None, FLinearColor::Black);

				// key diamond at end
				FSlateDrawElement::MakeBox(OutDrawElements, LayerId,
					AllottedGeometry.ToPaintGeometry(EventSize, FSlateLayoutTransform(FVector2D(XEndDiamond, Y))), EventBrush, ESlateDrawEffect::None, Window.Color);

				FSlateDrawElement::MakeBox(OutDrawElements, LayerId,
					AllottedGeometry.ToPaintGeometry(EventSize, FSlateLayoutTransform(FVector2D(XEndDiamond, Y))), EventBorderBrush, ESlateDrawEffect::None, FLinearColor::Black);
			}
		}

		LayerId++;
	}

	return LayerId;
}

#undef LOCTEXT_NAMESPACE
