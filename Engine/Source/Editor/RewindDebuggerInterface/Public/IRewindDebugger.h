// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "TraceServices/Model/Frames.h"
#include "UObject/NameTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "RewindDebuggerTypes.h"
#include "IRewindDebugger.generated.h"

REWINDDEBUGGERINTERFACE_API DECLARE_LOG_CATEGORY_EXTERN(LogRewindDebugger, Log, All);

namespace TraceServices
{
	class IAnalysisSession;
}

namespace UE::TraceBasedDebuggers
{
	struct FSessionInfo;
}

namespace RewindDebugger
{
	class FRewindDebuggerTrack;
	class FPropertiesTrack;
}
class IGameplayProvider;
struct FObjectInfo;

struct FDebugObjectInfo
{
	FDebugObjectInfo() = default;
	FDebugObjectInfo(const RewindDebugger::FObjectId& Id, const FString& Name)
		: Id(Id)
		, ObjectName(Name)
		, bExpanded(true)
	{
	}

	/** @return Part of the Id representing the UObject */
	uint64 GetUObjectId() const
	{
		return Id.GetMainId();
	}

	RewindDebugger::FObjectId Id;
	FString ObjectName;
	TArray<TSharedPtr<FDebugObjectInfo>> Children;
	bool bExpanded = false;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "Use ObjectIdentifier instead")
	uint64 ObjectId = 0;
#endif
};

struct FScrubTimeInformation
{
	/** Elapsed simulation/world time */
	double ElapsedTime = 0;
	/** Profiling/Tracing time */
	double ProfileTime = 0;
	/** Scrub Frame Index */
	int64 FrameIndex = 0;
};

UCLASS(MinimalAPI)
class URewindDebuggerTrackContextMenuContext : public UObject
{
	GENERATED_BODY()
public:
	TSharedPtr<FDebugObjectInfo> SelectedObject;
	TArray<FName> TypeHierarchy;
	TSharedPtr<RewindDebugger::FRewindDebuggerTrack> SelectedTrack;
};

class REWINDDEBUGGERINTERFACE_API UE_DEPRECATED(5.7, "Use URewindDebuggerTrackContextMenuContext instead") UComponentContextMenuContext : public URewindDebuggerTrackContextMenuContext
{
};

/**
 * IRewindDebugger
 * Public interface to rewind debugger
 */
class IRewindDebugger
{
public:
	REWINDDEBUGGERINTERFACE_API IRewindDebugger();

	REWINDDEBUGGERINTERFACE_API virtual ~IRewindDebugger();

	/** Whether the recording can be started. */
	virtual bool CanStartRecording() const = 0;

	/** Start traces recording if not already started. */
	virtual void StartRecording() const = 0;

	/** @return the time the debugger is scrubbed to, in seconds since the capture started (or the recording duration while the game is running) */
	virtual double CurrentTraceTime() const = 0;

	/** @return the time the debugger is scrubbed to, in seconds since the recording started */
	virtual double GetScrubTime() const = 0;

	/** Moves the scrub cursor to Time (seconds since the recording started), clamping to the recording range and disabling auto-scroll-with-recording. */
	virtual void SetScrubTime(double Time) = 0;

	/** Re-centers the visible view range on Time, preserving the current view width. Independent of the scrub cursor. */
	virtual void CenterViewOnTime(double Time) = 0;

	/** @return the current visible range in trace/profiler units (same units as CurrentTraceTime) */
	virtual const TRange<double>& GetCurrentTraceRange() const = 0;

	/** @return the current visible range in Rewind Debugger recording time units */
	virtual const TRange<double>& GetCurrentViewRange() const = 0;

	/**
	 * Returns shared pointer to the selected debug session (PIE or remote).
	 * @return the currently selected debug session
	 */
	virtual TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo> GetSelectedDebugSessionInfo() const = 0;

	/**
	 * Returns pointer to the current analysis sessions for immediate access.
	 * If the goal is to store in a variable for later access then GetAnalysisSessionAsShared should be used instead.
	 * @return the current analysis session
	 * @see GetAnalysisSessionAsShared
	 */
	virtual const TraceServices::IAnalysisSession* GetAnalysisSession() const = 0;

	/**
	 * Returns shared pointer to the current analysis sessions.
	 * @return the current analysis session
	 */
	virtual TSharedPtr<const TraceServices::IAnalysisSession> GetAnalysisSessionAsShared() const = 0;

	/** @return insights id for the root object currently debugged */
	virtual uint64 GetRootObjectId() const = 0;

	UE_DEPRECATED(5.7, "Use GetRootObjectId instead")
	virtual uint64 GetTargetActorId() const final
	{
		return 0; //GetDebuggedObjectId();
	}

	/** @return list of all objects associated to the active tracks */
	virtual TArray<TSharedPtr<FDebugObjectInfo>>& GetDebuggedObjects() = 0;

	UE_DEPRECATED(5.7, "Use GetDebuggedObjects instead")
	virtual TArray<TSharedPtr<FDebugObjectInfo>>& GetDebugComponents() final
	{
		return GetDebuggedObjects();
	}

	/**
	 * Sets the object to display tracks for.
	 * Method will do nothing uf the specified object is already part of the current object hierarchy.
	 * @param ObjectId The Id of the object to debug
	 */
	virtual void SetObjectToDebug(RewindDebugger::FObjectId ObjectId) = 0;

	/** @return Whether the given object id is one of the object currently debugged, or one of its children. */
	virtual bool IsObjectCurrentlyDebugged(uint64 InObjectId) const = 0;

	UE_DEPRECATED(5.7, "Use IsObjectCurrentlyDebugged instead")
	virtual bool IsContainedByDebugComponent(uint64 InObjectId) const final
	{
		return IsObjectCurrentlyDebugged(InObjectId);
	}

	/** @return the currently selected debug object */
	virtual TSharedPtr<FDebugObjectInfo> GetSelectedObject() const = 0;

	/** @return the currently selected track */
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> GetSelectedTrack() const = 0;

	/** Try selecting the track associated to a given object */
	virtual void SelectTrack(RewindDebugger::FObjectId ObjectId) = 0;

	/** @return position of the root object being debugged (returns true if position is valid) */
	virtual bool GetRootObjectPosition(FVector& OutPosition) const = 0;

	/** Stores position of the root object being debugged */
	virtual void SetRootObjectPosition(const TOptional<FVector>& InPosition) = 0;

	UE_DEPRECATED(5.7, "Use GetRootObjectPosition instead")
	virtual bool GetTargetActorPosition(FVector& OutPosition) const final
	{
		return GetRootObjectPosition(OutPosition);
	}

	/** @return the world that the debugger is replaying in */
	virtual UWorld* GetWorldToVisualize() const = 0;

	/** @return whether the recording is active */
	virtual bool IsRecording() const = 0;

	/** @return whether a trace file is loaded from disk */
	UE_DEPRECATED(5.8, "Method is no longer used and will be removed.")
	virtual bool IsTraceFileLoaded() const final
	{
		return false;
	}

	/** @return PIE is running and not paused */
	virtual bool IsPIESimulating() const = 0;

	/** @return the length of the current recording */
	virtual double GetRecordingDuration() const = 0;

	/** Opens the Rewind Debugger details panel tab */
	virtual void OpenDetailsPanel() = 0;

	/** @return the object information of the first object inheriting from the specified type in the outer chain of ObjectId */
	template <typename T>
	const FObjectInfo* FindTypedOuterInfo(TNotNull<const IGameplayProvider*> GameplayProvider, uint64 ObjectId) const
	{
		return FindTypedOuterInfo(T::StaticClass(), GameplayProvider, ObjectId);
	}

	/** @return the object information of the first object inheriting from the specified type in the outer chain of ObjectId */
	virtual const FObjectInfo* FindTypedOuterInfo(TNotNull<const UStruct*> Type, TNotNull<const IGameplayProvider*> GameplayProvider, uint64 ObjectId) const = 0;

	UE_DEPRECATED(5.7, "Use FindTypedOuterInfo instead")
	virtual const FObjectInfo* FindOwningActorInfo(const IGameplayProvider* GameplayProvider, uint64 ObjectId) const final
	{
		return nullptr;
	}

	virtual bool ShouldDisplayWorld(uint64 WorldId) = 0;

	/** @return the current IRewindDebugger instance */
	static REWINDDEBUGGERINTERFACE_API IRewindDebugger* Instance();

protected:
	static REWINDDEBUGGERINTERFACE_API IRewindDebugger* InternalInstance;
};
