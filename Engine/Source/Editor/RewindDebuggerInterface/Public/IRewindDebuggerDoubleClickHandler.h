// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRewindDebugger.h"
#include "Features/IModularFeatures.h"

#define UE_API REWINDDEBUGGERINTERFACE_API

// Interface class for adding custom double click handling to components
class IRewindDebuggerDoubleClickHandler : public IModularFeature
{
public:
	static UE_API const FName ModularFeatureName;
	
	// returns the name of a type of UObject for which this menu extender will be called
	virtual FName GetTargetTypeName() const = 0;

	// called when the selected component has been double clicked.  return true if something was opened
	virtual bool HandleDoubleClick(IRewindDebugger* IRewindDebugger) = 0;
};

#undef UE_API
