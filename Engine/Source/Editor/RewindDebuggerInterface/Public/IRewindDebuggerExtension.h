// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Features/IModularFeatures.h"

#define UE_API REWINDDEBUGGERINTERFACE_API

class IRewindDebugger;

/**
 * Interface class for extensions which add functionality to the rewind debugger.
 * These will get updated on scrub/playback to handle things like updating the world state
 * to match recorded data from that time for a particular system.
 */
class IRewindDebuggerExtension : public IModularFeature
{
public:
	static UE_API const FName ModularFeatureName;

	/** debugging name for this extension */
	virtual FString GetName()
	{
		return TEXT("RewindDebugger Extension");
	}

	/** called while scrubbing, playing back, or paused */
	virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger)
	{
	}

	/** called when opening an analysis session */
	virtual void AnalysisSessionOpened(IRewindDebugger* RewindDebugger)
	{
	}

	/** called when closing an analysis session */
	virtual void AnalysisSessionClosed(IRewindDebugger* RewindDebugger)
	{
	}

	UE_DEPRECATED(5.8, "Use AnalysisSessionOpened instead or implement an extension inheriting from IRewindDebuggerRuntimeExtension")
	virtual void RecordingStarted(IRewindDebugger* RewindDebugger) final
	{
	};

	UE_DEPRECATED(5.8, "Use AnalysisSessionClosed instead or implement an extension inheriting from IRewindDebuggerRuntimeExtension")
	virtual void RecordingStopped(IRewindDebugger* RewindDebugger) final
	{
	};

	/** called when recording is unloaded */
	virtual void Clear(IRewindDebugger* RewindDebugger)
	{
	};

	// called when tracks get removed or added to the debugger
	virtual void OnTrackListChanged(IRewindDebugger* RewindDebugger)
	{
	};
};

#undef UE_API
