// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointerFwd.h"

class FName;
class FSpawnTabArgs;
class SDockTab;

class IRewindDebuggerModule : public IModuleInterface
{
public:
	/** @return the name of the main RewindDebugger window that could be used to invoke the associated tab in the Editor. */
	virtual FName GetMainTabName() const = 0;

	/** @return the name of the RewindDebugger details window that could be used to invoke the associated tab in the Editor. */
	virtual FName GetDetailsTabName() const = 0;

	/** @return the name of the main toolbar in the RewindDebugger window that could be used to add new entries. */
	virtual FName GetMainToolbarName() const = 0;

	/** @return the name of the Preview dropdown menu. Register menu extensions against it. */
	virtual FName GetPreviewMenuName() const = 0;

	virtual TSharedRef<SDockTab> SpawnRewindDebuggerTab(const FSpawnTabArgs& SpawnTabArgs) = 0;
	virtual TSharedRef<SDockTab> SpawnRewindDebuggerDetailsTab(const FSpawnTabArgs& SpawnTabArgs) = 0;
};