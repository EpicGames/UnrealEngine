// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreMinimal.h"
#include "Textures/SlateIcon.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7

#include "IRewindDebuggerView.h"

namespace TraceServices
{
	class IAnalysisSession;
}

class IRewindDebugger;

namespace RewindDebugger
{

class FRewindDebuggerTrack;
struct FObjectId;
struct FRewindDebuggerTrackType
{
	FName Name;
	FText DisplayName;
};

/**
 * Interface class which creates tracks
 */
class IRewindDebuggerTrackCreator : public IModularFeature
{
public:
	static REWINDDEBUGGERINTERFACE_API const FName ModularFeatureName;

	virtual ~IRewindDebuggerTrackCreator() = default;

	/** @return name of the type of UObject this track creator can create tracks for */
	FName GetTargetTypeName() const
	{
		return GetTargetTypeNameInternal();
	}

	/** @return identifying Name for this type of track */
	FName GetName() const
	{
		return GetNameInternal();
	}

	/** @return integer representing the track priority.  Higher values will show higher in the track list (default is 0) */
	int32 GetSortOrderPriority() const
	{
		return GetSortOrderPriorityInternal();
	}

	/**
	 * Optional filter to prevent tracks from being listed if there is no data associated to the provided object identifier.
	 * @return Whether the track should be created to represent data associated to the provided object identifier.
	 */
	bool HasDebugInfo(const FObjectId& ObjectId) const
	{
		return HasDebugInfoInternal(ObjectId);
	};

	UE_DEPRECATED(5.7, "Use the version taking FObjectId instead")
	bool HasDebugInfo(uint64 ObjectId) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return HasDebugInfoInternal(ObjectId);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	/**
	 * Creates a track which will be shown in the timeline view and tree view, as a child track of the Object
	 * @return the created child track
	 */
	TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrack(const FObjectId& ObjectId) const
	{
		return CreateTrackInternal(ObjectId);
	}

	UE_DEPRECATED(5.7, "Use the version taking FObjectId instead")
	TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrack(uint64 ObjectId) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CreateTrackInternal(ObjectId);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void GetTrackTypes(TArray<FRewindDebuggerTrackType>& Types) const
	{
		return GetTrackTypesInternal(Types);
	};

	/** @return Whether the track created will be used by its parent track to replace its representation and timeline values */
	bool IsCreatingPrimaryChildTrack() const
	{
		return IsCreatingPrimaryChildTrackInternal();
	}

private:

	/** get the UObject type name this Creator will create child tracks for */
	virtual FName GetTargetTypeNameInternal() const
	{
		return "Object";
	}

	/** get the Name (unique identifier) for this Creator */
	virtual FName GetNameInternal() const
	{
		return FName();
	}

	/** An integer to override sort order for tracks created by this Creator (Higher priority will make tracks appear higher in the list) */
	virtual int32 GetSortOrderPriorityInternal() const
	{
		return 0;
	}

	/** Add track types that this Creator will create to the track type list */
	virtual void GetTrackTypesInternal(TArray<FRewindDebuggerTrackType>& Types) const
	{
	};

	/** Returns true if this creator's track should be shown for this Object, false if there is no data and they should be hidden. */
	virtual bool HasDebugInfoInternal(const FObjectId& InObjectId) const
	{
		return true;
	};

	UE_DEPRECATED(5.7, "Use the version taking FObjectId instead")
	virtual bool HasDebugInfoInternal(uint64 InObjectId) const final
	{
		return true;
	};

	virtual TSharedPtr<FRewindDebuggerTrack> CreateTrackInternal(const FObjectId& InObjectId) const
	{
		return TSharedPtr<FRewindDebuggerTrack>();
	}

	UE_DEPRECATED(5.7, "Use the version taking FObjectId instead")
	virtual TSharedPtr<FRewindDebuggerTrack> CreateTrackInternal(uint64 InObjectId) const final
	{
		return TSharedPtr<FRewindDebuggerTrack>();
	}

	virtual bool IsCreatingPrimaryChildTrackInternal() const
	{
		return false;
	}
};

} // namespace RewindDebugger
