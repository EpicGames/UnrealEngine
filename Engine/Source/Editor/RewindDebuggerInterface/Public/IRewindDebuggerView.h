// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

// Interface class for views of recorded debug data
class IRewindDebuggerView : public SCompoundWidget
{
	public:
		// unique name for widget
		virtual FName GetName() const = 0;

		// id of target object
		virtual uint64 GetObjectId() const = 0;

		// called by the debugger when the scrubbing bar position changes
		virtual void SetTimeMarker(double InTimeMarker) = 0;
};
