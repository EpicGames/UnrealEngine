// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRewindDebuggerView.h"
#include "Textures/SlateIcon.h"

#define UE_API REWINDDEBUGGERINTERFACE_API

namespace TraceServices
{
	class IAnalysisSession;
}

// Interface class which creates debug widgets
class IRewindDebuggerViewCreator : public IModularFeature
{
public: 
	static UE_API const FName ModularFeatureName;

	// returns the name of a type of UObject for which this debug view will be created
	virtual FName GetTargetTypeName() const = 0;
	
	// optional additional filter, to prevent debug views from being listed if they have no data
	virtual bool HasDebugInfo(uint64 ObjectId) const
	{
		return true;
	};

	// returns a unique name for identifying this type of widget (same value returned by IRewindDebuggerView::GetName)
	virtual FName GetName() const = 0;

	// text for tab header
	virtual FText GetTitle() const = 0;

	// icon for tab header
	virtual FSlateIcon GetIcon() const = 0;

	// creates and returns a widget, which will be displayed in Rewind Debugger
	virtual TSharedPtr<IRewindDebuggerView> CreateDebugView(uint64 ObjectId, double CurrentTime, const TraceServices::IAnalysisSession& InAnalysisSession) const = 0;
};

#undef UE_API
