// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7

#include "IRewindDebugger.h"
#include "Widgets/SWidget.h"
#include "Textures/SlateIcon.h"

namespace TraceServices
{
	class IAnalysisSession;
}

struct FToolMenuSection;

namespace RewindDebugger
{
enum class ETrackHiddenFlags : uint8
{
	None = 0,
	HiddenByCode = 1 << 0,
	HiddenByUI = 1 << 1,
	AnyFlag = 0xFF
};
ENUM_CLASS_FLAGS(ETrackHiddenFlags);

enum class EStepMode : uint8
{
	Forward,
	Backward
};

class FRewindDebuggerTrack
{
public:
	/** Result returned from a visitor functor indicating to continue or to quit early */
	enum class EVisitResult : uint8
	{
		/** Stop iterating through tracks and early exit */
		Break,
		/**  Continue to iterate through all tracks */
		Continue
	};

	FRewindDebuggerTrack()
	{
	}

	virtual ~FRewindDebuggerTrack() = default;

	bool GetIsExpanded() const
	{
		return bExpanded;
	}

	void SetIsExpanded(const bool bIsExpanded)
	{
		bExpanded = bIsExpanded;
	}

	bool GetIsSelected() const
	{
		return bSelected;
	}

	void SetIsSelected(const bool bIsSelected)
	{
		bSelected = bIsSelected;
	}

	bool GetIsTreeHovered() const
	{
		return bTreeHovered;
	}

	void SetIsTreeHovered(const bool bIsHovered)
	{
		bTreeHovered = bIsHovered;
	}

	bool GetIsTrackHovered() const
	{
		return bTrackHovered;
	}

	void SetIsTrackHovered(const bool bIsHovered)
	{
		bTrackHovered = bIsHovered;
	}

	bool GetIsHovered() const
	{
		return bTrackHovered || bTreeHovered;
	}

	/** Update should do work to compute children etc. for the current time range.  Return true if children have changed. */
	bool Update()
	{
		return UpdateInternal();
	}

	/** Get a widget to show in the timeline view for this track */
	TSharedPtr<SWidget> GetTimelineView()
	{
		return GetTimelineViewInternal();
	}

	/** Get a widget to show in the details tab, when this track is selected */
	TSharedPtr<SWidget> GetDetailsView()
	{
		return GetDetailsViewInternal();
	}

	/** unique name for track (must match creator name if track is created by an IRewindDebuggerViewCreator)  */
	FName GetName() const
	{
		return GetNameInternal();
	}

	/** icon to display in the tree view */
	FSlateIcon GetIcon()
	{
		return GetIconInternal();
	}

	/** display name for track in Tree View */
	FText GetDisplayName() const
	{
		return GetDisplayNameInternal();
	}

	/** insights main UObject id for an object associated with this track */
	uint64 GetUObjectId() const
	{
		return GetObjectIdInternal();
	}

	UE_DEPRECATED(5.7, "Use GetUObjectID instead.")
	uint64 GetObjectId() const
	{
		return GetUObjectId();
	}

	/** insights object id for an object associated with this track */
	FObjectId GetAssociatedObjectId() const
	{
		return FObjectId(GetObjectIdInternal());
	}

	/** tracks can customize the tooltip for the step command */
	FText GetStepCommandTooltip(const EStepMode StepMode) const
	{
		return GetStepCommandTooltipInternal(StepMode);
	}

	UE_DEPRECATED(5.8, "Use overload taking a scrub time information struct instead.")
	/** tracks can return a different time for the step action; default the main recording frame time is used */
	TOptional<double> GetStepFrameTime(const EStepMode StepMode, const double CurrentScrubTime) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GetStepFrameTimeInternal(StepMode, CurrentScrubTime);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** tracks can return a different time for the step action; default the main recording frame time is used */
	TOptional<double> GetStepFrameTime(const EStepMode StepMode, const FScrubTimeInformation& CurrentScrubTime) const
	{
		return GetStepFrameTimeInternal(StepMode, CurrentScrubTime);
	}

	/**
	 * Visit the current track along with its child hierarchy recursively
	 * to call provided function on each track until a call returns EVisitResult::Break
	 */
	static EVisitResult Visit(TSharedPtr<FRewindDebuggerTrack> Track, TFunction<EVisitResult(TSharedPtr<FRewindDebuggerTrack> SubTrack)> IteratorFunction)
	{
		if (IteratorFunction(Track) == EVisitResult::Break)
		{
			return EVisitResult::Break;
		}

		return Track->Visit(IteratorFunction);
	}

private:
	/**
	 * Visit the child hierarchy recursively to call provided function on each track until a call returns EVisitResult::Break
	 */
	EVisitResult Visit(TFunction<EVisitResult(TSharedPtr<FRewindDebuggerTrack> SubTrack)> IteratorFunction)
	{
		// GetChildrenInternal can provide children using the returned array view,
		// the output parameter, or both.
		TArray<TSharedPtr<FRewindDebuggerTrack>> Children;
		for (const TSharedPtr<FRewindDebuggerTrack>& Track : GetChildrenInternal(Children))
		{
			if (Visit(Track, IteratorFunction) == EVisitResult::Break)
			{
				return EVisitResult::Break;
			}
		}

		for (const TSharedPtr<FRewindDebuggerTrack>& Track : Children)
		{
			if (Visit(Track, IteratorFunction) == EVisitResult::Break)
			{
				return EVisitResult::Break;
			}
		}

		return EVisitResult::Continue;
	}
public:

	/** iterates over all sub-tracks of this track and call Iterator function */
	void IterateSubTracks(TFunction<void(TSharedPtr<FRewindDebuggerTrack> SubTrack)> IteratorFunction) const
	{
		// GetChildrenInternal can provide children using the returned array view,
		// the output parameter, or both.
		TArray<TSharedPtr<FRewindDebuggerTrack>> Children;
		for (const TSharedPtr<FRewindDebuggerTrack>& Track : GetChildrenInternal(Children))
		{
			IteratorFunction(Track);
		}

		for (const TSharedPtr<FRewindDebuggerTrack>& Track : Children)
		{
			IteratorFunction(Track);
		}
	}

	/** returns true for tracks that contain debug data (used for filtering out parts of the hierarchy with no useful information in them) */
	bool HasDebugData() const
	{
		return HasDebugDataInternal();
	}

	/** Called when a track is double-clicked.  Returns true if the track handled the double click */
	bool HandleDoubleClick()
	{
		return HandleDoubleClickInternal();
	}

	void SetHiddenFlag(const ETrackHiddenFlags Flag)
	{
		EnumAddFlags(HiddenFlags, Flag);
	}

	void UnsetHiddenFlag(const ETrackHiddenFlags Flag)
	{
		EnumRemoveFlags(HiddenFlags, Flag);
	}

	bool HasAnyFlags(const ETrackHiddenFlags Flags) const
	{
		return EnumHasAnyFlags(HiddenFlags, Flags);
	}

	bool IsVisible() const
	{
		return HiddenFlags == ETrackHiddenFlags::None;
	}

	UE_DEPRECATED(5.7, "Use SetHiddenFlag/UnsetHiddenFlag instead.")
	void SetIsVisible(const bool bInIsVisible)
	{
		if (bInIsVisible)
		{
			UnsetHiddenFlag(ETrackHiddenFlags::HiddenByCode);
		}
		else
		{
			SetHiddenFlag(ETrackHiddenFlags::HiddenByCode);
		}
	}

	/** Called to generate context menu for the current selected track */
	virtual void BuildContextMenu(FToolMenuSection& InMenuSection) {};

private:

	virtual bool UpdateInternal()
	{
		return false;
	}

	virtual TSharedPtr<SWidget> GetTimelineViewInternal()
	{
		return TSharedPtr<SWidget>();
	}

	virtual TSharedPtr<SWidget> GetDetailsViewInternal()
	{
		return TSharedPtr<SWidget>();
	}

	virtual FSlateIcon GetIconInternal()
	{
		return FSlateIcon();
	}

	virtual FName GetNameInternal() const
	{
		return "";
	}

	virtual FText GetDisplayNameInternal() const
	{
		return FText();
	}

	virtual uint64 GetObjectIdInternal() const
	{
		return {};
	}

	virtual bool HasDebugDataInternal() const
	{
		return true;
	}

	virtual FText GetStepCommandTooltipInternal(EStepMode) const
	{
		return {};
	}

	UE_DEPRECATED(5.8, "Use overload taking a scrub time information struct instead.")
	virtual TOptional<double> GetStepFrameTimeInternal(EStepMode, const double) const final
	{
		return {};
	}

	virtual TOptional<double> GetStepFrameTimeInternal(EStepMode, const FScrubTimeInformation&) const
	{
		return {};
	}

	UE_DEPRECATED(5.7, "Implement and use GetChildrenInternal instead.")
	virtual void IterateSubTracksInternal(TFunction<void(TSharedPtr<FRewindDebuggerTrack> SubTrack)> IteratorFunction) final
	{
	}

	/**
	 * Method that derive classes can override to provide children tracks for methods like Visit and IterateSubTracks
	 * Caller of this method is expected to handle both the return value and the output parameter.
	 * @param OutTracks Optional parameter that can be used when the derived class can't provide a const view of the children
	 * @return An array view on the sub tracks
	 */
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
	{
		return {};
	}

	virtual bool HandleDoubleClickInternal()
	{
		IRewindDebugger::Instance()->OpenDetailsPanel();
		return true;
	};

	ETrackHiddenFlags HiddenFlags = ETrackHiddenFlags::None;

	bool bSelected : 1 = false;
	bool bTrackHovered : 1 = false;
	bool bTreeHovered : 1 = false;
	bool bExpanded : 1 = true;
};

} // namespace RewindDebugger
