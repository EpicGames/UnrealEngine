// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/TypeHash.h"
#include "Containers/UnrealString.h"

namespace RewindDebugger
{

/**
 * Struct representing an object (single UObject or a child element of a UObject) tracked by Unreal Insight traces.
 * This struct is used to represent the data traced by UE::ObjectTrace::FObjectId
 */
struct FObjectId
{
	static constexpr uint64 InvalidId = 0;

	FObjectId() = default;

	explicit FObjectId(const uint64 MainId)
		: ObjectId(MainId)
	{
	}

	/**
	 * Identifier associated to a UObject recorded using TRACE_OBJECT* macros.
	 * @return Identifier associated to a given UObject in traces.
	 */
	uint64 GetMainId() const
	{
		return ObjectId;
	}

	/**
	 * @return Whether the current gameplay object identifier was set to represent a UObject, or one of its children.
	 */
	bool IsSet() const
	{
		return ObjectId != InvalidId;
	}

	bool operator==(const FObjectId& Other) const = default;

	bool operator<(const FObjectId& Other) const
	{
		return ObjectId < Other.ObjectId;
	}

	friend FString LexToString(const FObjectId& ObjectIdentifier)
	{
		if (ObjectIdentifier.ObjectId == InvalidId)
		{
			return TEXT("Invalid");
		}

		return FString::Printf(TEXT("[%llX]"), ObjectIdentifier.ObjectId);
	}

	friend uint32 GetTypeHash(const FObjectId Identifier)
	{
		return GetTypeHash(Identifier.ObjectId);
	}

private:
	uint64 ObjectId = InvalidId;
};

}