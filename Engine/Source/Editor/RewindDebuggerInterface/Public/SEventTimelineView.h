// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API REWINDDEBUGGERINTERFACE_API

class FPaintArgs;
class FSlateWindowElementList;
class FTimeSliderController;

class SEventTimelineView : public SCompoundWidget
{
public:
	struct FTimelineEventData
	{
		struct FEventPoint
		{
			double Time;
			FText Type;
			FText Description;
			FLinearColor Color;
		};

		struct FEventWindow
		{
			double TimeStart;
			double TimeEnd;
			FText Type;
			FText Description;
			FLinearColor Color;
		};

		TArray<FEventPoint> EventPoints;
		TArray<FEventWindow> EventWindows;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITORONLY_DATA
		struct UE_DEPRECATED(5.8, "Use FEventPoint instead") EventPoint
		{
			double Time;
			FText Type;
			FText Description;
			FLinearColor Color;
		};
		struct UE_DEPRECATED(5.8, "Use FEventWindow instead") EventWindow
		{
			double TimeStart;
			double TimeEnd;
			FText Type;
			FText Description;
			FLinearColor Color;
		};

		UE_DEPRECATED(5.8, "Use EventPoints instead")
		TArray<EventPoint> Points;
		UE_DEPRECATED(5.8, "Use EventWindows instead")
		TArray<EventWindow> Windows;
#endif
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	SLATE_BEGIN_ARGS(SEventTimelineView)
		: _ViewRange(TRange<double>(0, 10))
		, _DesiredSize(FVector2D(100.f, 20.f))
	{
	}
	/** View time range */
	SLATE_ATTRIBUTE(TRange<double>, ViewRange);

	/** Data for events to render */
	SLATE_ATTRIBUTE(TSharedPtr<FTimelineEventData>, EventData);

	/** Desired widget size */
	SLATE_ATTRIBUTE(FVector2D, DesiredSize);

	SLATE_END_ARGS()


	/**
	 * Construct the widget
	 *
	 * @param InArgs A declaration from which to construct the widget
	 */
	UE_API void Construct(const FArguments& InArgs);

protected:
	//~ SWidget interface
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual int32 OnPaint(const FPaintArgs& Args
		, const FGeometry& AllottedGeometry
		, const FSlateRect& MyCullingRect
		, FSlateWindowElementList& OutDrawElements
		, int32 LayerId
		, const FWidgetStyle& InWidgetStyle
		, bool bParentEnabled) const override;

	UE_API int32 PaintEvents(const FGeometry& AllottedGeometry
		, const FSlateRect& MyCullingRect
		, FSlateWindowElementList& OutDrawElements
		, int32 LayerId
		, const FWidgetStyle& InWidgetStyle
		, bool bParentEnabled) const;

	TAttribute<TRange<double>> ViewRange;
	TAttribute<TSharedPtr<FTimelineEventData>> EventData;
	TAttribute<FVector2D> DesiredSize;

	const FSlateBrush* EventBrush = nullptr;
	const FSlateBrush* EventBorderBrush = nullptr;
};

#undef UE_API
