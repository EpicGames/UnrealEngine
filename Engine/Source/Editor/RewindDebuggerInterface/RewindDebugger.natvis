<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!--
  *
  * RewindDebugger Visualizers
  *
  -->

  <Type Name="RewindDebugger::FObjectId">
    <DisplayString>{ObjectId}</DisplayString>
  </Type>

  <Type Name="FClassInfo">
    <DisplayString Condition="SuperId == 0">Id={Id,x} {Name,su}</DisplayString>
    <DisplayString Condition="SuperId != 0">Id={Id,x} SuperId={SuperId,x} {Name,su}</DisplayString>
  </Type>

  <Type Name="FObjectInfo">
    <DisplayString Condition="OuterObjectId.ObjectId == 0">Id={ObjectId,x} {Name,su}</DisplayString>
    <DisplayString Condition="OuterObjectId.ObjectId != 0">Id={ObjectId,x} OuterId={OuterObjectId,x} {Name,su}</DisplayString>
  </Type>

  <Type Name="FWorldInfo">
    <DisplayString Condition="PIEInstanceId == -1">{Type} {NetMode} Id={Id,x} Simulating={bIsSimulating}</DisplayString>
    <DisplayString Condition="PIEInstanceId != -1">{Type}:{PIEInstanceId} {NetMode} Id={Id,x} Simulating={bIsSimulating}</DisplayString>
  </Type>
</AutoVisualizer>